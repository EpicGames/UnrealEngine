// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraitCore/TraitBinding.h"
#include "TraitCore/TraitSharedData.h"
#include "MirroringTask.h"

#include "MirroringTraitData.generated.h"

/** A trait that can mirror an input's keyframe data. */
USTRUCT(meta = (DisplayName = "Mirroring", ShowTooltip = true))
struct FMirroringTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	// Input node to query the keyframe to be mirrored.
	UPROPERTY()
	FAnimNextTraitHandle Input;

	// Defines whether to perform mirror pass (and what data table to use).
	UPROPERTY(EditAnywhere, Category = "Setup", meta=(ShowOnlyInnerProperties))
	FUAFMirroringTraitSetupParams Setup;

	// Defines what channels will be affected during the mirror pass.
	UPROPERTY(EditAnywhere, Category = "Apply To", meta=(ShowOnlyInnerProperties))
	FUAFMirroringTraitApplyToParams ApplyTo;
	
	// Latent pin support boilerplate
	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
	GeneratorMacro(Input) \
	GeneratorMacro(Setup) \
	GeneratorMacro(ApplyTo) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FMirroringTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

/** Same behaviour as FMirroringTrait, but as an additive (i.e. it only mirrors the super-trait’s output). */
USTRUCT(meta = (DisplayName = "Mirroring", ShowTooltip = true))
struct FMirroringAdditiveTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	// Defines whether to perform mirror pass (and what data table to use).
	UPROPERTY(EditAnywhere, Category = "Setup", meta=(ShowOnlyInnerProperties))
	FUAFMirroringTraitSetupParams Setup;

	// Defines what channels will be affected during the mirror pass.
	UPROPERTY(EditAnywhere, Category = "Apply To", meta=(ShowOnlyInnerProperties))
	FUAFMirroringTraitApplyToParams ApplyTo;
	
	// Latent pin support boilerplate
	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
	GeneratorMacro(Setup) \
	GeneratorMacro(ApplyTo) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FMirroringAdditiveTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

namespace UE::UAF
{
	// Namespaced alias
	using FMirroringTraitData = FMirroringTraitSharedData;
	using FMirroringAdditiveTraitData = FMirroringAdditiveTraitSharedData;
	
}