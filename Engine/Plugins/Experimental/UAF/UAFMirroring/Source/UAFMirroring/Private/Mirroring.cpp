// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mirroring.h"

#include "EvaluationVM/EvaluationVM.h"
#include "TransformArrayOperations.h"
#include "Animation/MirrorDataTable.h"
#include "LODPose.h"
#include "AnimationRuntime.h"
#include "Engine/DataTable.h"
#include "Logging/StructuredLog.h"
#include "Engine/SkeletalMesh.h"

namespace UE::UAF
{
	using FMemStackSetAllocator = TSetAllocator<TSparseArrayAllocator<FAnimStackAllocator, FAnimStackAllocator>, FAnimStackAllocator>;

	int32 GetNumOfBonesForMirrorData(const FReferencePose& InReferencePose)
	{
		return InReferencePose.GetNumBonesForLOD(0);
	}
	
	void BuildReferencePoseMirrorData(
		const FReferencePose& InReferencePose,
		EAxis::Type InMirrorAxis,
		TConstArrayView<FBoneIndexType> InMeshBoneIndexToMirroredMeshBoneIndexMap,
		TArrayView<FQuat> OutRefPoseMeshSpaceRotations,
		TArrayView<FQuat> OutRefPoseMeshSpaceRotationCorrections)
	{
		const int32 RefPoseBoneNum = InReferencePose.ReferenceLocalTransforms.Num();
		
		const TArrayView<const FBoneIndexType>& MeshBoneIndexToParentMeshBoneIndexMap = InReferencePose.MeshBoneIndexToParentMeshBoneIndexMap;

		// All the reference arrays are sized of LOD0 since we can always truncate the arrays if we need higher LOD levels. 
		checkf(OutRefPoseMeshSpaceRotations.Num() == RefPoseBoneNum, TEXT("Buffer mismatch: %d:%d"), OutRefPoseMeshSpaceRotations.Num() , RefPoseBoneNum);
		checkf(OutRefPoseMeshSpaceRotationCorrections.Num() == RefPoseBoneNum, TEXT("Buffer mismatch: %d:%d"), OutRefPoseMeshSpaceRotationCorrections.Num(), RefPoseBoneNum);
		checkf(InMeshBoneIndexToMirroredMeshBoneIndexMap.Num() == RefPoseBoneNum, TEXT("Buffer mismatch: %d:%d"), InMeshBoneIndexToMirroredMeshBoneIndexMap.Num(), RefPoseBoneNum)
		checkf(MeshBoneIndexToParentMeshBoneIndexMap.Num() == RefPoseBoneNum, TEXT("Buffer mismatch: %d:%d"), MeshBoneIndexToParentMeshBoneIndexMap.Num(), RefPoseBoneNum);
		
		// Copy local ref pose rotations.
		FMemory::Memcpy(OutRefPoseMeshSpaceRotations.GetData(), InReferencePose.ReferenceLocalTransforms.Rotations.GetData(), RefPoseBoneNum * sizeof(FQuat));

		// Convert the local ref pose rotations to component space.
		// @note: We assume the root bone is at index 0.
		for (int32 MeshBoneIndex = 1; MeshBoneIndex < RefPoseBoneNum; ++MeshBoneIndex)
		{
			const FBoneIndexType LODBoneIndex = InReferencePose.MeshBoneIndexToLODBoneIndexMap[MeshBoneIndex];
			const FBoneIndexType ParentMeshBoneIndex = MeshBoneIndexToParentMeshBoneIndexMap[MeshBoneIndex];
			
			const FQuat ParentLocalSpaceRotation = OutRefPoseMeshSpaceRotations[ParentMeshBoneIndex];
			const FQuat RefBoneLocalSpaceRotation = InReferencePose.ReferenceLocalTransforms[LODBoneIndex].GetRotation();
			FQuat MeshSpaceRotation =  ParentLocalSpaceRotation * RefBoneLocalSpaceRotation;
			MeshSpaceRotation.Normalize();
			
			OutRefPoseMeshSpaceRotations[MeshBoneIndex] = MeshSpaceRotation;
		}

		// Now we can precompute the corrective rotation to align result with target space's rest orientation.
		OutRefPoseMeshSpaceRotationCorrections[0] = FQuat::Identity;
		for (FBoneIndexType MeshBoneIndex = 1; MeshBoneIndex < RefPoseBoneNum; ++MeshBoneIndex)
		{
			const FBoneIndexType SourceBoneIndex = MeshBoneIndex;
			const FBoneIndexType TargetBoneIndex = InMeshBoneIndexToMirroredMeshBoneIndexMap[SourceBoneIndex];

			// Not mapped, so skip.
			if (TargetBoneIndex == INVALID_BONE_INDEX)
			{
				continue;
			}
			
			OutRefPoseMeshSpaceRotationCorrections[MeshBoneIndex] = FAnimationRuntime::MirrorQuat(OutRefPoseMeshSpaceRotations[SourceBoneIndex], InMirrorAxis).Inverse() * OutRefPoseMeshSpaceRotations[TargetBoneIndex];
			OutRefPoseMeshSpaceRotationCorrections[MeshBoneIndex].Normalize();
		}
	}
	
	// Get a map from source bone index to mirrored bone index.
	void BuildMeshBoneIndexMirrorMap(
		const FReferencePose& InReferencePose,
		const UMirrorDataTable& InMirrorDataTable,
		TArrayView<FBoneIndexType> OutMeshBoneIndexToMirroredMeshBoneIndexMap)
	{
		// Add this in case this code is called outside update/eval pass in UAF.
		FMemMark Mark(FMemStack::Get());
		
		const int32 RefPoseBoneNum = InReferencePose.ReferenceLocalTransforms.Num();
		
		// All the reference arrays are sized of LOD0 since we can always truncate the arrays if we need higher LOD levels. 
		checkf(RefPoseBoneNum == OutMeshBoneIndexToMirroredMeshBoneIndexMap.Num(), TEXT("Buffer mismatch: %d:%d"), RefPoseBoneNum, OutMeshBoneIndexToMirroredMeshBoneIndexMap.Num());
		
		// Reset the mirror table to defaults (no mirroring).
		FMemory::Memset(OutMeshBoneIndexToMirroredMeshBoneIndexMap.GetData(), INDEX_NONE, RefPoseBoneNum * OutMeshBoneIndexToMirroredMeshBoneIndexMap.GetTypeSize());

		// Query only bone info from mirror data table.
		TMap<FName, FName, TInlineSetAllocator<128, FMemStackSetAllocator>> NameToMirrorNameBoneMap;
		InMirrorDataTable.ForeachRow<FMirrorTableRow>(TEXT("UE::UAF::FillMirrorBoneIndexes"), [&NameToMirrorNameBoneMap](const FName& Key, const FMirrorTableRow& Value) mutable
			{
				if (Value.MirrorEntryType == EMirrorRowType::Bone)
				{
					NameToMirrorNameBoneMap.Add(Value.Name, Value.MirroredName);
				}
			}
		);

		// We need the reference skeleton to be able to query bone names.
		// @todo: Will this change once we have abstract hierarchies?
		const USkeletalMesh* SkeletalMesh = InReferencePose.GetSkeletalMeshAsset();
		checkf(SkeletalMesh != nullptr, TEXT("UAF::FillMirrorBoneIndices - SkeletalMesh is null"));
		const FReferenceSkeleton& ReferenceSkeleton = SkeletalMesh->GetRefSkeleton();
		
		const USkeleton* Skeleton = SkeletalMesh->GetSkeleton(); 
		const FReferenceSkeleton& FullLODSkeleton = Skeleton->GetReferenceSkeleton(); 

		const TArrayView<const FBoneIndexType> LODBoneIndexToSkeletonBoneIndexMap = InReferencePose.GetLODBoneIndexToSkeletonBoneIndexMap(0);
		const TArrayView<const FBoneIndexType> MeshBoneIndexToLODBoneIndexMap = InReferencePose.GetMeshBoneIndexToLODBoneIndexMap();
		const TMap<FName, FBoneIndexType> BoneNameToLODBoneIndexMap = InReferencePose.GetBoneNameToLODBoneIndexMap();
		const TArrayView<const FBoneIndexType> LODBoneIndexToMeshBoneIndexMap = InReferencePose.GetLODBoneIndexToMeshBoneIndexMap(0);

		checkf(LODBoneIndexToSkeletonBoneIndexMap.Num() == RefPoseBoneNum, TEXT("Buffer mismatch: %d:%d"), LODBoneIndexToSkeletonBoneIndexMap.Num(), RefPoseBoneNum);
		checkf(MeshBoneIndexToLODBoneIndexMap.Num() == RefPoseBoneNum, TEXT("Buffer mismatch: %d:%d"), MeshBoneIndexToLODBoneIndexMap.Num(), RefPoseBoneNum);
		checkf(BoneNameToLODBoneIndexMap.Num() == RefPoseBoneNum, TEXT("Buffer mismatch: %d:%d"), BoneNameToLODBoneIndexMap.Num(), RefPoseBoneNum);
		checkf(LODBoneIndexToMeshBoneIndexMap.Num() == RefPoseBoneNum, TEXT("Buffer mismatch: %d:%d"), LODBoneIndexToMeshBoneIndexMap.Num(), RefPoseBoneNum);

		// Build mirror map for all the mesh bones in reference pose (LOD0).
		for (FBoneIndexType MeshBoneIndex = 0; MeshBoneIndex < OutMeshBoneIndexToMirroredMeshBoneIndexMap.Num(); ++MeshBoneIndex)
		{
			if (OutMeshBoneIndexToMirroredMeshBoneIndexMap[MeshBoneIndex] == INVALID_BONE_INDEX)
			{
				// Get index for LOD sorted map
				const FBoneIndexType LODBoneIndex = MeshBoneIndexToLODBoneIndexMap[MeshBoneIndex];
				checkf(LODBoneIndex != INVALID_BONE_INDEX, TEXT("Failed to find LOD Bone Index counterpart for Mesh Bone Index"))

				// We need to convert from our sorted LOD bone map to the skeleton bone map, otherwise the mapping is broken.
				const FBoneIndexType SkeletonBoneIndex = LODBoneIndexToSkeletonBoneIndexMap[LODBoneIndex];
				
				// since this is a skeleton index we need to use the skeleton to find the name 
				// Find the candidate mirror partner for this bone
				const FName SourceBoneName = FullLODSkeleton.GetBoneName(SkeletonBoneIndex);
				FBoneIndexType MirrorMeshBoneIndex = INVALID_BONE_INDEX;
				
				FName* MirroredBoneName = NameToMirrorNameBoneMap.Find(SourceBoneName);
				if (!SourceBoneName.IsNone() && MirroredBoneName && BoneNameToLODBoneIndexMap.Contains(*MirroredBoneName))
				{
					int32 MirrorLODBoneIndex = BoneNameToLODBoneIndexMap[*MirroredBoneName];

					if (MirrorLODBoneIndex != INVALID_BONE_INDEX)
					{
						MirrorMeshBoneIndex = LODBoneIndexToMeshBoneIndexMap[MirrorLODBoneIndex];
					}
				}
				
				OutMeshBoneIndexToMirroredMeshBoneIndexMap[MeshBoneIndex] = MirrorMeshBoneIndex;
				
				// Map candidate mirror partner to current bone.
				// @todo: What happens we have conflicts on mirror data table setup. Should we have a conflict policy to resolve this? For example, two bones map to another.
				if (MirrorMeshBoneIndex != INVALID_BONE_INDEX)
				{
					OutMeshBoneIndexToMirroredMeshBoneIndexMap[MirrorMeshBoneIndex] = MeshBoneIndex;
				}
			}
		}
	}

	void MirrorPose(
		FLODPose& InOutLODPose,
		const UMirrorDataTable& InMirrorDataTable)
	{
		const int32 SourceLODBoneNum = InOutLODPose.LocalTransformsView.Num();
		
		if (InMirrorDataTable.MirrorAxis == EAxis::None)
		{
			UE_LOGF(LogAnimation, Warning, "UAF::MirrorPose - No mirror axis provided.");
			return;
		}

		if (SourceLODBoneNum == 0)
		{
			UE_LOGF(LogAnimation, Warning, "UAF::MirrorPose - Attempting to mirror an empty pose.");
			return;
		}

		if (InOutLODPose.IsAdditive())
		{
			UE_LOGF(LogAnimation, Warning, "UAF::MirrorPose - Attempting to mirror an additive pose.");
			return;
		}
		
		// Get mirrored bones mapping.
		TArray<FBoneIndexType, FAnimStackAllocator> MeshBoneIndexToMirroredMeshBoneIndexMap;
		TArray<FQuat, FAnimStackAllocator> MeshSpaceReferencePoseRotations;
		TArray<FQuat, FAnimStackAllocator> MeshSpaceReferenceRotationCorrections;

		const FReferencePose& ReferencePose = InOutLODPose.GetRefPose();
		
		const int32 MirrorBoneIndicesNum = GetNumOfBonesForMirrorData(ReferencePose);
		MeshBoneIndexToMirroredMeshBoneIndexMap.SetNumUninitialized(MirrorBoneIndicesNum);

		const int32 BindPoseMirrorDataNum = GetNumOfBonesForMirrorData(InOutLODPose.GetRefPose());
		MeshSpaceReferencePoseRotations.SetNumUninitialized(BindPoseMirrorDataNum);
		MeshSpaceReferenceRotationCorrections.SetNumUninitialized(BindPoseMirrorDataNum);
		
		BuildMeshBoneIndexMirrorMap(ReferencePose, InMirrorDataTable, MeshBoneIndexToMirroredMeshBoneIndexMap);
		BuildReferencePoseMirrorData(ReferencePose, InMirrorDataTable.MirrorAxis, MeshBoneIndexToMirroredMeshBoneIndexMap, MeshSpaceReferencePoseRotations, MeshSpaceReferenceRotationCorrections);
		
		MirrorPose(InOutLODPose, InMirrorDataTable.MirrorAxis, MeshBoneIndexToMirroredMeshBoneIndexMap, MeshSpaceReferencePoseRotations, MeshSpaceReferenceRotationCorrections);
	}
	
	void MirrorPose(
		FLODPose& InOutLODPose,
		EAxis::Type InMirrorAxis,
		TConstArrayView<FBoneIndexType> InMeshBoneIndexToMirroredMeshBoneIndexMap,
		TConstArrayView<FQuat> InRefPoseMeshSpaceRotations,
		TConstArrayView<FQuat> InRefPoseMeshSpaceRotationCorrections)
	{
		const int32 SourceLODBoneNum = InOutLODPose.LocalTransformsView.Num();
		
		const int32 RefPoseBoneNum = InOutLODPose.GetRefPose().ReferenceLocalTransforms.Num();
		checkf(RefPoseBoneNum == InMeshBoneIndexToMirroredMeshBoneIndexMap.Num(), TEXT("Buffer mismatch: %d:%d"), RefPoseBoneNum, InMeshBoneIndexToMirroredMeshBoneIndexMap.Num());
		
		if (InMirrorAxis == EAxis::None )
		{
			UE_LOGF(LogAnimation, Warning, "UAF::MirrorPose - No mirror axis provided.");
			return;
		}

		if (SourceLODBoneNum == 0)
		{
			UE_LOGF(LogAnimation, Warning, "UAF::MirrorPose - Attempting to mirror an empty pose.");
			return;
		}

		if (InOutLODPose.IsAdditive())
		{
			UE_LOGF(LogAnimation, Warning, "UAF::MirrorPose - Attempting to mirror an additive pose.");
			return;
		}
		
		const TArrayView<const FBoneIndexType>& LODBoneIndexToParentLODBoneIndexMap = InOutLODPose.GetLODBoneIndexToParentLODBoneIndexMap();
		checkf(LODBoneIndexToParentLODBoneIndexMap.Num() == InOutLODPose.GetNumBones(), TEXT("Buffer mismatch: %d:%d"), LODBoneIndexToParentLODBoneIndexMap.Num(), InOutLODPose.GetNumBones());

		const TArrayView<const FBoneIndexType>& LODBoneIndexToMeshBoneIndexMap = InOutLODPose.GetLODBoneIndexToMeshBoneIndexMap();
		checkf(LODBoneIndexToMeshBoneIndexMap.Num() == InOutLODPose.GetNumBones(), TEXT("Buffer mismatch: %d:%d"), LODBoneIndexToMeshBoneIndexMap.Num(), InOutLODPose.GetNumBones());

		const TArrayView<const FBoneIndexType>& MeshBoneIndexToLODBoneIndexMap = InOutLODPose.GetMeshBoneIndexToLODBoneIndexMap();
		checkf(MeshBoneIndexToLODBoneIndexMap.Num() == InRefPoseMeshSpaceRotationCorrections.Num(), TEXT("Buffer mismatch: %d:%d"), MeshBoneIndexToLODBoneIndexMap.Num(), InRefPoseMeshSpaceRotationCorrections.Num())
		
		// Mirroring is authored in object space and as such we must transform the local space transforms in object space in order
		// to apply the object space mirroring axis. To facilitate this, we use object space transforms for the bind pose which can be cached.
		// We ignore the translation/scale part of the bind pose as they don't impact mirroring.
		// 
		// Rotations, translations, and scales are all treated differently:
		//    Rotation:
		//        We transform the local space rotation into object space
		//        We mirror the rotation axis
		//        We apply a correction: if the source and target bones are different, we must account for the mirrored delta between the two
		//        We transform the result back into local space
		//    Translation:
		//        We rotate the local space translation into object space
		//        We mirror the result
		//        We then rotate it back into local space
		//    Scale:
		//        Mirroring does not modify scale
		// 
		// This sadly doesn't quite work for additive poses because in order to transform it into the bind pose reference frame,
		// we need the base pose it is applied on. Worse still, the base pose might not be static, it could be a time scaled sequence.
		//
		// Contract/Assumption: the (SourceBoneIndex -> TargetBoneIndex) mapping used to call this matches the mapping used to build OutMirrorCorrections (i.e., Target == MirrorMap[Source]).
		const auto MirrorTransform = [&InRefPoseMeshSpaceRotations, &InMirrorAxis, InRefPoseMeshSpaceRotationCorrections](const FTransform& SourceTransform, const FBoneIndexType& SourceParentIndex, const FBoneIndexType& SourceBoneIndex, const FBoneIndexType& TargetParentIndex, const FBoneIndexType& TargetBoneIndex) -> FTransform
		{
			const FQuat TargetParentRefRotation = TargetParentIndex != INVALID_BONE_INDEX ? InRefPoseMeshSpaceRotations[TargetParentIndex] : FQuat::Identity;
			const FQuat SourceParentRefRotation = SourceParentIndex != INVALID_BONE_INDEX ? InRefPoseMeshSpaceRotations[SourceParentIndex] : FQuat::Identity;

			// Mirror the translation component: Rotate the translation into the space of the mirror plane, mirror across the plane, and rotate into the space of its new parent.

			FVector T = SourceTransform.GetTranslation();
			T = SourceParentRefRotation.RotateVector(T);          // to mesh space (component space)
			T = FAnimationRuntime::MirrorVector(T, InMirrorAxis); // reflect across plane
			T = TargetParentRefRotation.UnrotateVector(T);        // back to target's parent local space

			// Mirror the rotation component: Rotate into the space of the mirror plane, mirror across the plane, apply corrective rotation to align result with target space's rest orientation, 
			// then rotate into the space of its new parent
				
			FQuat Q = SourceTransform.GetRotation();
			Q = SourceParentRefRotation * Q;                     // to mesh space (component space)
			Q = FAnimationRuntime::MirrorQuat(Q, InMirrorAxis);  // reflect accros plane

			// Bind alignment correction: mirror the source bone's bind orientation and align to target bone's bind orientation.
			// @note: we use a precomputed correction quat to be able to save a MirrorQuat() since this can be cached once per skeleton,
			// this assumes that source always is the mesh bone index and target always is the mirrored mesh bone index.
			Q *= InRefPoseMeshSpaceRotationCorrections[SourceBoneIndex];
			Q = TargetParentRefRotation.Inverse() * Q;
			Q.Normalize();

			// Scale is not affected by mirroring.
			FVector S = SourceTransform.GetScale3D();

			return FTransform(Q, T, S);
		};
		
		// Mirror the root bone.
		{
			// @todo: Can the root bone be other than 0? LODPose.GetRefPose().RootBoneIndex? Can we have more than one root bone? 
			const FBoneIndexType RootMeshBoneIndex = 0;
			const FBoneIndexType RootLODBoneIndex =  InMeshBoneIndexToMirroredMeshBoneIndexMap[RootMeshBoneIndex];
			const FBoneIndexType MirrorRootMeshBoneIndex = InMeshBoneIndexToMirroredMeshBoneIndexMap[RootMeshBoneIndex];

			if (MirrorRootMeshBoneIndex != INVALID_BONE_INDEX && RootLODBoneIndex != INVALID_BONE_INDEX)
			{
				const FBoneIndexType MirroredRootLODBoneIndex = MeshBoneIndexToLODBoneIndexMap[MirrorRootMeshBoneIndex];

				if (MirroredRootLODBoneIndex == INVALID_BONE_INDEX)
				{
					UE_LOGFMT(LogAnimation, Error, "UAF::MirrorPose - Failed to find LOD Bone index ({0}) for Mesh Bone Index ({1}) for a mirrored root bone. This is invalid.", MirroredRootLODBoneIndex, MirrorRootMeshBoneIndex);
					return;
				}
				
				const FBoneIndexType TargetParentLODBoneIndex = LODBoneIndexToParentLODBoneIndexMap[RootMeshBoneIndex];
				const FBoneIndexType SourceParentLODBoneIndex = LODBoneIndexToParentLODBoneIndexMap[MirroredRootLODBoneIndex];

				if (TargetParentLODBoneIndex != INVALID_BONE_INDEX)
				{
					UE_LOGFMT(LogAnimation, Error, "UAF::MirrorPose - Found parent bone index ({0}) for root bone index ({1}). This is invalid.", TargetParentLODBoneIndex, RootMeshBoneIndex);
					return;
				}

				if (SourceParentLODBoneIndex != INVALID_BONE_INDEX)
				{
					UE_LOGFMT(LogAnimation, Error, "UAF::MirrorPose - Found parent bone index ({0}) for mirrored root bone index ({1}). This is invalid.", TargetParentLODBoneIndex, RootMeshBoneIndex);
					return;
				}

				// At this point the parent bone of the root should be -1 so we're good to just set it as is.
				const FBoneIndexType TargetParentMeshBoneIndex = TargetParentLODBoneIndex;
				const FBoneIndexType SourceParentMeshBoneIndex = SourceParentLODBoneIndex;
				
				InOutLODPose.LocalTransformsView[RootLODBoneIndex] = MirrorTransform(InOutLODPose.LocalTransformsView[RootLODBoneIndex], SourceParentMeshBoneIndex, RootMeshBoneIndex, TargetParentMeshBoneIndex, RootMeshBoneIndex);
			}
		}
		
		// Mirror the non-root bones.
		for (FBoneIndexType LODBoneIndex = 1; LODBoneIndex < SourceLODBoneNum; ++LODBoneIndex)
		{
			// Skip invalid mesh bone index (should never happen).
			const FBoneIndexType MeshBoneIndex = LODBoneIndex != INVALID_BONE_INDEX ? LODBoneIndexToMeshBoneIndexMap[LODBoneIndex] : INVALID_BONE_INDEX;
			if (MeshBoneIndex == INVALID_BONE_INDEX)
			{
				continue;
			}
				
			// @note: We can safely use the LODBoneIndex to get our mirrored name since the map is in LOD0 which will always contain the index.
			const FBoneIndexType MirroredMeshBoneIndex = InMeshBoneIndexToMirroredMeshBoneIndexMap[MeshBoneIndex];

			// Skip invalid mirrors.
			const bool bIsMirrorInvalid = MirroredMeshBoneIndex == INVALID_BONE_INDEX;
			if (bIsMirrorInvalid)
			{
				continue;	
			}
			
			const FBoneIndexType MirroredLODBoneIndex = MeshBoneIndexToLODBoneIndexMap[MirroredMeshBoneIndex];
			const FBoneIndexType ParentLODBoneIndex = LODBoneIndexToParentLODBoneIndexMap[LODBoneIndex];
			const FBoneIndexType ParentMeshBoneIndex = ParentLODBoneIndex != INVALID_BONE_INDEX ? LODBoneIndexToMeshBoneIndexMap[ParentLODBoneIndex] : INVALID_BONE_INDEX;
			
			// Self-mirror (mirror in place relative to the same parent).
			const bool bIsMirroredBoneMappedToSelf = LODBoneIndex == MirroredLODBoneIndex;
			if (bIsMirroredBoneMappedToSelf)
			{
				InOutLODPose.LocalTransformsView[LODBoneIndex] = MirrorTransform(InOutLODPose.LocalTransformsView[LODBoneIndex], ParentMeshBoneIndex, MeshBoneIndex, ParentMeshBoneIndex, MeshBoneIndex);
				continue;
			}

			// Skip invalid or already-processed mirror pairs.
			const bool bIsAlreadyProcessed = MirroredLODBoneIndex < LODBoneIndex;
			if (bIsAlreadyProcessed)
			{
				continue;
			}
			
			const FBoneIndexType MirroredParentLODBoneIndex = LODBoneIndexToParentLODBoneIndexMap[MirroredLODBoneIndex];
			const FBoneIndexType MirroredParentMeshBoneIndex = LODBoneIndexToMeshBoneIndexMap[MirroredParentLODBoneIndex];
			
			const FTransform OriginalTransformAtBoneIndex = InOutLODPose.LocalTransformsView[LODBoneIndex];
			const FTransform OriginalTransformAtMirroredIndex = InOutLODPose.LocalTransformsView[MirroredLODBoneIndex];
			
			const FTransform NewTransformAtBoneIndex = MirrorTransform(OriginalTransformAtMirroredIndex, MirroredParentMeshBoneIndex, MirroredMeshBoneIndex, ParentMeshBoneIndex, MeshBoneIndex);
			const FTransform NewTransformAtMirroredIndex = MirrorTransform(OriginalTransformAtBoneIndex, ParentMeshBoneIndex, MeshBoneIndex, MirroredParentMeshBoneIndex, MirroredMeshBoneIndex);
			
			InOutLODPose.LocalTransformsView[LODBoneIndex] = NewTransformAtBoneIndex;
			InOutLODPose.LocalTransformsView[MirroredLODBoneIndex] = NewTransformAtMirroredIndex;
		}
	}
}
