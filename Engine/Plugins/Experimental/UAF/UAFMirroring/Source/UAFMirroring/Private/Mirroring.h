// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LODPose.h"

namespace UE::UAF
{
	/** Number of mesh bones (use this when pre-sizing the arrays for bind pose mirror data and mirror map). */
	int32 GetNumOfBonesForMirrorData(const FReferencePose & InReferencePose);
	
	/**
	 * Build the bind-pose data needed to mirror a pose.
	 * *
	 * @param InReferencePose Reference/bind pose context. Used to access bind pose local transforms and parent maps.
	 * @param InMirrorAxis Mirror axis (X/Y/Z)
	 * @param InMeshBoneIndexToMirroredMeshBoneIndexMap Array view of mirrored mesh bone indices
	 * @param OutRefPoseMeshSpaceRotations A pre-sized array view to fill reference pose rotations in mesh space
	 * @param OutRefPoseMeshSpaceRotationCorrections A pre-sized array view to fill reference pose rotation corrections in mesh space
	 * 
	 * @note Output arrays are expected to be pre-sized to the number of bones in the reference pose's skeleton (i.e. LOD0).
	 */
	void BuildReferencePoseMirrorData(
		const FReferencePose& InReferencePose,
		EAxis::Type InMirrorAxis,
		TConstArrayView<FBoneIndexType> InMeshBoneIndexToMirroredMeshBoneIndexMap,
		TArrayView<FQuat> OutRefPoseMeshSpaceRotations,
		TArrayView<FQuat> OutRefPoseMeshSpaceRotationCorrections);

	/**
	 * Build a mesh-bone mirror map from a mirror data table.
	 * 
	 * @param InReferencePose Reference/bind pose context. Used to query a mesh bone index by name.
	 * @param InMirrorDataTable Data table with mirror bone name pairs (and mirror axis)
	 * @param OutMeshBoneIndexToMirroredMeshBoneIndexMap A pre-sized array view to fill with mirrored mesh bone indices.
	 * 
	 * @note Output map is expected to be pre-sized to the number of bones in the reference pose's skeleton (i.e. LOD0)
	 */
	void BuildMeshBoneIndexMirrorMap(
		const FReferencePose& InReferencePose,
		const UMirrorDataTable& InMirrorDataTable,
		TArrayView<FBoneIndexType> OutMeshBoneIndexToMirroredMeshBoneIndexMap);
	
	/**
	 * Mirror a pose in place using a mirror data table.
	 * 
	 * @param InOutLODPose Pose to mirror (local-space)
	 * @param InMirrorDataTable Data table with mirror bone name pairs (and mirror axis)
	 */
	void MirrorPose(
		FLODPose& InOutLODPose,
		const UMirrorDataTable& InMirrorDataTable);
	
	/**
	 * Mirror a pose in place using explicit mirror axis + precomputed bind-pose data.
	 * 
	 * @param InOutLODPose Pose to mirror (local-space)
	 * @param InMirrorAxis Mirror axis (X/Y/Z)
	 * @param InMeshBoneIndexToMirroredMeshBoneIndexMap Mapping from mesh bone index to its mirrored counterpart (if any)
	 * @param InRefPoseMeshSpaceRotations Reference pose rotations in mesh space
	 * @param InRefPoseMeshSpaceRotationCorrections Reference pose rotation corrections in mesh space
	 * 
	 * @note Input map and arrays are expected to be the size of to the number of bones in the reference pose's skeleton (i.e. LOD0)
	 */
	void MirrorPose(
		FLODPose& InOutLODPose,
		EAxis::Type InMirrorAxis,
		TConstArrayView<FBoneIndexType> InMeshBoneIndexToMirroredMeshBoneIndexMap,
		TConstArrayView<FQuat> InRefPoseMeshSpaceRotations,
		TConstArrayView<FQuat> InRefPoseMeshSpaceRotationCorrections);
}
