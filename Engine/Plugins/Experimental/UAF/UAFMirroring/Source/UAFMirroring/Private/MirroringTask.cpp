// Copyright Epic Games, Inc. All Rights Reserved.

#include "MirroringTask.h"

#include "GenerationTools.h"
#include "EvaluationVM/EvaluationVM.h"
#include "TransformArrayOperations.h"
#include "Animation/MirrorDataTable.h"
#include "LODPose.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"
#include "AnimationRuntime.h"
#include "Mirroring.h"
#include "BoneContainer.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FAnimNextEvaluationMirroringTask

void FAnimNextEvaluationMirroringTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	QUICK_SCOPE_CYCLE_COUNTER(FAnimNextEvaluationMirroringTask_Execute);

	if (VM.GetActiveNamedSet())
	{
		// TODO: Implement with new attribute runtime
		return;
	}

	if (!Setup.MirrorDataTable)
	{
		return;
	}

	// Pop our top pose and start processing it.
	TUniquePtr<UE::UAF::FKeyframeState> Keyframe;
	if (!VM.PopValue(UE::UAF::KEYFRAME_STACK_NAME, Keyframe))
	{
		// We have no inputs, nothing to do.
		return;
	}

	// We don't support mirroring additive pose at the moment.
	if (Keyframe->Pose.IsAdditive())
	{
		UE_LOGF(LogAnimation, Warning, "FAnimNextEvaluationMirroringTask::Execute - Mirroring an additive pose is not supported.")
	
		// Force a bind pose to make it obvious.
		Keyframe->Pose.SetIdentity(true);
		VM.PushValue(UE::UAF::KEYFRAME_STACK_NAME, MoveTemp(Keyframe));
		return;
	}

	// Rebuild mirror cache (if needed)
	UE::UAF::FReferencePose ReferencePose = Keyframe->Pose.GetRefPose();
	EnsureCache(VM, ReferencePose);

	// Determine what channel to skip during this pass.
	bool bShouldMirrorBones = ApplyTo.bShouldMirrorBones && EnumHasAnyFlags(VM.GetFlags(), UE::UAF::EEvaluationFlags::Bones);
	bool bShouldMirrorCurves= ApplyTo.bShouldMirrorCurves && EnumHasAnyFlags(VM.GetFlags(), UE::UAF::EEvaluationFlags::Curves);
	bool bShouldMirrorAttributes = ApplyTo.bShouldMirrorAttributes && EnumHasAnyFlags(VM.GetFlags(), UE::UAF::EEvaluationFlags::Attributes);

	// After attempting to rebuild cache if its still invalid something went wrong.
	if (!Cache.IsValid(ReferencePose, Setup.MirrorDataTable, bShouldMirrorBones, bShouldMirrorAttributes))
	{
		// Force a bind pose to make it obvious.
		Keyframe->Pose.SetIdentity(Keyframe->Pose.IsAdditive());
		VM.PushValue(UE::UAF::KEYFRAME_STACK_NAME, MoveTemp(Keyframe));
		return;
	}

	// Mirror keyframe data.

	if (bShouldMirrorBones)
	{
		UE::UAF::MirrorPose(
			Keyframe->Pose,
			Setup.MirrorDataTable->MirrorAxis,
			Cache.MeshBoneIndexToMirroredMeshBoneIndexMap,
			Cache.MeshSpaceReferencePoseRotations,
			Cache.MeshSpaceReferenceRotationCorrections);
	}

	if (bShouldMirrorCurves)
	{
		FAnimationRuntime::MirrorCurves(Keyframe->Curves, *Setup.MirrorDataTable);
	}

	VM.PushValue(UE::UAF::KEYFRAME_STACK_NAME, MoveTemp(Keyframe));
}

void FAnimNextEvaluationMirroringTask::EnsureCache(const UE::UAF::FEvaluationVM& VM, const UE::UAF::FReferencePose& InReferencePose) const
{
	if (InReferencePose.SkeletalMesh.IsValid())
	{
		const int32 NumBonesForLOD0 = UE::UAF::GetNumOfBonesForMirrorData(InReferencePose);
		const bool bShouldMirrorBones = ApplyTo.bShouldMirrorBones && EnumHasAnyFlags(VM.GetFlags(), UE::UAF::EEvaluationFlags::Bones);
		const bool bShouldMirrorAttributes = ApplyTo.bShouldMirrorAttributes && EnumHasAnyFlags(VM.GetFlags(), UE::UAF::EEvaluationFlags::Attributes);
		const bool bAreMirrorMapsValid = Cache.AreMirrorMapsValid(InReferencePose, Setup.MirrorDataTable, bShouldMirrorBones, bShouldMirrorAttributes);
		const bool bIsReferencePoseDataValid = Cache.IsReferencePoseDataValid(InReferencePose, bShouldMirrorBones);
	
		if (bShouldMirrorBones)
		{
			if (!bAreMirrorMapsValid)
			{
				Cache.MeshBoneIndexToMirroredMeshBoneIndexMap.SetNumUninitialized(NumBonesForLOD0);
			
				UE::UAF::BuildMeshBoneIndexMirrorMap(InReferencePose, *Setup.MirrorDataTable, Cache.MeshBoneIndexToMirroredMeshBoneIndexMap);
			}
		
			if (!bIsReferencePoseDataValid)
			{
				Cache.MeshSpaceReferencePoseRotations.SetNumUninitialized(NumBonesForLOD0);
				Cache.MeshSpaceReferenceRotationCorrections.SetNumUninitialized(NumBonesForLOD0);
			
				UE::UAF::BuildReferencePoseMirrorData(
					InReferencePose,
					Setup.MirrorDataTable->MirrorAxis,
					Cache.MeshBoneIndexToMirroredMeshBoneIndexMap,
					Cache.MeshSpaceReferencePoseRotations,
					Cache.MeshSpaceReferenceRotationCorrections
				);
			}
		}
	
		if (bShouldMirrorAttributes)
		{
			if (!bAreMirrorMapsValid)
			{
				// We only care about the mesh bones (LOD0).
				TArray<FBoneIndexType> RequiredBoneIndexArray;
				RequiredBoneIndexArray.AddUninitialized(NumBonesForLOD0);
			
				for (int32 BoneIndex = 0; BoneIndex < RequiredBoneIndexArray.Num(); ++BoneIndex)
				{
					RequiredBoneIndexArray[BoneIndex] = InReferencePose.GetSkeletonBoneIndexFromLODBoneIndex(BoneIndex);
				}

				// Compute mirror map for compact pose.
			
				FBoneContainer BoneContainer;
				const USkeleton* Skeleton = InReferencePose.SkeletalMesh->GetSkeleton();
				BoneContainer.InitializeTo(RequiredBoneIndexArray, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::DisallowAll), *Skeleton);
			
				TCustomBoneIndexArray<FSkeletonPoseBoneIndex, FSkeletonPoseBoneIndex> MirrorBoneIndexes;
				Setup.MirrorDataTable->FillMirrorBoneIndexes(BoneContainer.GetSkeletonAsset(), MirrorBoneIndexes);
				Setup.MirrorDataTable->FillCompactPoseMirrorBones(BoneContainer, MirrorBoneIndexes, Cache.CompactPoseBoneIndexToMirroredCompactPoseBoneIndexMap);
			}
		}
	}
	else
	{
		// Clear data to terminate task early.
		Cache.Clear();
	
		UE_LOGF(LogAnimation, Warning, "FAnimNextEvaluationMirroringTask::EnsureCache - Failed to get skeletal mesh asset from reference pose.")
	}

	// Keep track of latest assets used to build the cache
	Cache.SkeletalMesh = InReferencePose.SkeletalMesh;
	Cache.MirrorTable = Setup.MirrorDataTable;
}