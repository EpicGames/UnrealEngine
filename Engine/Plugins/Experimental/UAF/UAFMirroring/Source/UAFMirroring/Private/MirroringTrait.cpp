// Copyright Epic Games, Inc. All Rights Reserved.

#include "MirroringTrait.h"

#include "GenerationTools.h"
#include "EvaluationVM/EvaluationVM.h"
#include "TransformArrayOperations.h"
#include "Animation/MirrorDataTable.h"
#include "LODPose.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"
#include "AnimationRuntime.h"
#include "Mirroring.h"
#include "BoneContainer.h"
#include "MirroringTask.h"

namespace UE::UAF
{
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	// FMirroringTraitCache


	bool FMirroringTraitCache::AreMirrorMapsValid(const UE::UAF::FReferencePose& InReferencePose, const TWeakObjectPtr<const UMirrorDataTable>& InMirrorTable, bool bInShouldMirrorBones, bool bInShouldMirrorAttributes) const
	{
		// Ensure assets are the same.

		if (InMirrorTable != MirrorTable)
		{
			return false;
		}

		if (InReferencePose.SkeletalMesh != SkeletalMesh)
		{
			return false;
		}

		// Ensure arrays match mesh bone count.

		const int32 ExpectedNumBones = UE::UAF::GetNumOfBonesForMirrorData(InReferencePose);

		// Default to true if we are not mirroring bones since these arrays are only used for that channel.
		const bool bDoesMeshMirrorMapHaveExpectedNumBones = (ExpectedNumBones == MeshBoneIndexToMirroredMeshBoneIndexMap.Num() || !bInShouldMirrorBones);

		// Default to true if we are not mirroring attributes since these arrays are only used for that channel.
		const bool bDoesCompactPoseMirrorMapHaveExpectedNumBones = (ExpectedNumBones == CompactPoseBoneIndexToMirroredCompactPoseBoneIndexMap.Num() || !bInShouldMirrorAttributes);

		const bool bAreMirrorMapsEquallySized = bDoesMeshMirrorMapHaveExpectedNumBones && bDoesCompactPoseMirrorMapHaveExpectedNumBones;
		if (!bAreMirrorMapsEquallySized)
		{
			return false;
		}

		// Default to true if we are not mirroring bones since these arrays are only used for that channel.
		const bool bIsRefPoseDataEquallySized = (ExpectedNumBones == MeshSpaceReferencePoseRotations.Num() && ExpectedNumBones == MeshSpaceReferenceRotationCorrections.Num()) || !bInShouldMirrorBones;
		if (!bIsRefPoseDataEquallySized)
		{
			return false;
		}

		return true;
	}
	
	bool FMirroringTraitCache::IsReferencePoseDataValid(const UE::UAF::FReferencePose& InReferencePose, bool bInShouldMirrorBones) const
	{
		// Ensure skeletal mesh is still the same.
		if (InReferencePose.SkeletalMesh != SkeletalMesh)
		{
			return false;
		}

		// Ensure arrays match mesh bone count.
		// Default to true if we are not mirroring bones since these arrays are only used for that channel.
		const int32 ExpectedNumBones = UE::UAF::GetNumOfBonesForMirrorData(InReferencePose);
		const bool bIsRefPoseDataEquallySized = (ExpectedNumBones == MeshSpaceReferencePoseRotations.Num() && ExpectedNumBones == MeshSpaceReferenceRotationCorrections.Num()) || !bInShouldMirrorBones;
		if (!bIsRefPoseDataEquallySized)
		{
			return false;
		}
		
		return true;
	}

	bool FMirroringTraitCache::IsValid(const UE::UAF::FReferencePose& InReferencePose, const TWeakObjectPtr<const UMirrorDataTable>& InMirrorTable, bool bInShouldMirrorBones, bool bInShouldMirrorAttributes) const
	{
		return AreMirrorMapsValid(InReferencePose, InMirrorTable, bInShouldMirrorBones, bInShouldMirrorAttributes) && IsReferencePoseDataValid(InReferencePose, bInShouldMirrorBones);
	}

	void FMirroringTraitCache::Clear()
	{
		MeshBoneIndexToMirroredMeshBoneIndexMap.Empty();
		CompactPoseBoneIndexToMirroredCompactPoseBoneIndexMap.Empty();
		MeshSpaceReferencePoseRotations.Empty();
		MeshSpaceReferenceRotationCorrections.Empty();
		MirrorTable.Reset();
		SkeletalMesh.Reset();
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	// FMirroringTrait
	
	AUTO_REGISTER_ANIM_TRAIT(FMirroringTrait)
	
	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
	GeneratorMacro(IEvaluate) \
	GeneratorMacro(IHierarchy) \
	GeneratorMacro(IGarbageCollection) \
	
	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FMirroringTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	void FMirroringTrait::FInstanceData::Construct(const FExecutionContext& Context, const FTraitBinding& Binding)
	{
		FTrait::FInstanceData::Construct(Context, Binding);
		
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		
		if (!InstanceData->Input.IsValid())
		{
			InstanceData->Input = Context.AllocateNodeInstance(Binding, SharedData->GetInput(Binding));
		}

		if (!InstanceData->Task.IsValid())
		{
			InstanceData->Task = MakeShared<FAnimNextEvaluationMirroringTask>();
		}
	}

	void FMirroringTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		IEvaluate::PostEvaluate(Context, Binding);
		
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		// Copy since the settings properties are 'latent' and can change..
		const FMirroringTraitSetupParams LatentSetupValue = SharedData->GetSetup(Binding);;

		InstanceData->bHasMirrorStateChanged = LatentSetupValue.bShouldMirror != InstanceData->Task->Setup.bShouldMirror;
		InstanceData->Task->Setup = LatentSetupValue;
		InstanceData->Task->ApplyTo = SharedData->GetApplyTo(Binding);

		if (!InstanceData->Task->Setup.bShouldMirror || !InstanceData->Input.IsValid() || !InstanceData->Task->Setup.MirrorDataTable)
		{
			return;
		}
		
		Context.AppendTaskPtr(InstanceData->Task);
	}

	uint32 FMirroringTrait::GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const
	{
		return 1;
	}

	void FMirroringTrait::GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		// Add the child, even if the handle is empty
		Children.Add(InstanceData->Input);
	}

	void FMirroringTrait::AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const
	{
		IGarbageCollection::AddReferencedObjects(Context, Binding, Collector);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		Collector.AddReferencedObject(InstanceData->Task->Setup.MirrorDataTable);
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	// FMirroringAdditiveTrait
	
	AUTO_REGISTER_ANIM_TRAIT(FMirroringAdditiveTrait)
	
	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
	GeneratorMacro(IEvaluate) \
	GeneratorMacro(IGarbageCollection) \
	
	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FMirroringAdditiveTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	void FMirroringAdditiveTrait::FInstanceData::Construct(const FExecutionContext& Context, const FTraitBinding& Binding)
	{
		FTrait::FInstanceData::Construct(Context, Binding);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		if (!InstanceData->Task.IsValid())
		{
			InstanceData->Task = MakeShared<FAnimNextEvaluationMirroringTask>();
		}
	}

	void FMirroringAdditiveTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		IEvaluate::PostEvaluate(Context, Binding);
		
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		// Copy since the settings properties are 'latent' and can change.
		const FMirroringTraitSetupParams LatentSetupValue = SharedData->GetSetup(Binding);;

		InstanceData->bHasMirrorStateChanged = LatentSetupValue.bShouldMirror != InstanceData->Task->Setup.bShouldMirror;
		InstanceData->Task->Setup = LatentSetupValue;
		InstanceData->Task->ApplyTo = SharedData->GetApplyTo(Binding);
		
		if (!InstanceData->Task->Setup.bShouldMirror || !InstanceData->Task->Setup.MirrorDataTable)
		{
			return;
		}
		
		Context.AppendTaskPtr(InstanceData->Task);
	}

	void FMirroringAdditiveTrait::AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const
	{
		IGarbageCollection::AddReferencedObjects(Context, Binding, Collector);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		Collector.AddReferencedObject(InstanceData->Task->Setup.MirrorDataTable);
	}
	
} // namespace UE::UAF
