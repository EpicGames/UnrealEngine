// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/KismetMathLibrary.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitSharedData.h"
#include "TraitInterfaces/IAttributeProvider.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IHierarchy.h"
#include "TraitInterfaces/IUpdate.h"
#include "TraitInterfaces/IGarbageCollection.h"
#include "MirroringTraitData.h"
#include "CustomBoneIndexArray.h"

class UMirrorDataTable;
class USkeletalMesh;

namespace UE::UAF
{
	/**
	 * Mirroring Base Trait
	 * 
	 * A trait that can mirror an input's keyframe data.
	 */
	struct FMirroringTrait : FBaseTrait, IEvaluate, IHierarchy, IGarbageCollection
	{
		DECLARE_ANIM_TRAIT(FMirroringTrait, FBaseTrait)

		using FSharedData = FMirroringTraitSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
			// Input node to query the keyframe to be mirrored.
			FTraitPtr Input;

			// Task containing the setup data on how to do the mirroring
			TSharedPtr<FAnimNextEvaluationMirroringTask> Task;
			
			// Used to trigger inertial blends.
			bool bHasMirrorStateChanged = false;
			
			void Construct(const FExecutionContext& Context, const FTraitBinding& Binding);
		};
		
		// IEvaluate impl
		virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;
		
		// IHierarchy impl
		virtual uint32 GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const override;
		virtual void GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const override;
		
		// IGarbageCollection impl
		virtual void AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const override;
	};

	/**
	 * Mirroring Additive Trait
	 * 
	 * Same behaviour as FMirroringTrait, but as an additive (i.e. it only mirrors the super-trait’s output).
	 */
	struct FMirroringAdditiveTrait : FAdditiveTrait, IUpdate, IEvaluate, IGarbageCollection
	{
		DECLARE_ANIM_TRAIT(FMirroringAdditiveTrait, FAdditiveTrait)

		using FSharedData = FMirroringAdditiveTraitSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
			TSharedPtr<FAnimNextEvaluationMirroringTask> Task;
				
			// Used to trigger inertial blends.
			bool bHasMirrorStateChanged = false;

			void Construct(const FExecutionContext& Context, const FTraitBinding& Binding);
			
		};

		// IEvaluate impl
		virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;

		// IGarbageCollection impl
		virtual void AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const override;
	};
	
} // namespace UE::UAF