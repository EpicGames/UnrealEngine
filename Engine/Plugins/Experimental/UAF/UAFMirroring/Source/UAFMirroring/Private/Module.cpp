// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "MirroringTraitData.h"
#include "Animation/MirrorDataTable.h"
#include "Module/AnimNextModule.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "RigVMCore/RigVMRegistry.h"

namespace UE::UAF::Mirroring
{

class FModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		static TPair<UClass*, FRigVMRegistry::ERegisterObjectOperation> const AllowedObjectTypes[] =
		{
			{ UMirrorDataTable::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
		};

		static UScriptStruct* const AllowedStructTypes[] =
		{
			FMirroringTraitSetupParams::StaticStruct(),
			FMirroringTraitApplyToParams::StaticStruct(),
		};
		
		FRigVMRegistry::Get().RegisterObjectTypes(AllowedObjectTypes);
		FRigVMRegistry::Get().RegisterStructTypes(AllowedStructTypes);
	}

	virtual void ShutdownModule() override
	{
	}
};

}

IMPLEMENT_MODULE(UE::UAF::Mirroring::FModule, UAFMirroring)