// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EvaluationVM/EvaluationTask.h"
#include "ReferencePose.h"
#include "CustomBoneIndexArray.h"
#include "MirroringTask.generated.h"

class UMirrorDataTable;

/**
	 * Mirroring Setup Parameters
	 *
	 * This struct holds the minimal configuration needed to enable a mirror pass.
	 *
	 * @see UMirrorDataTable, FMirroringTrait, FMirroringAdditiveTrait
	 */
USTRUCT(BlueprintType, meta = (DisplayName = "Mirroring Trait Setup Parameters"))
struct FUAFMirroringTraitSetupParams
{
	GENERATED_BODY()

	// Whether to perform mirror pass
	UPROPERTY(EditAnywhere, Category = Settings, meta=(DisplayName = "Mirror"))
	bool bShouldMirror = true;

	// Data table to map bones to their mirrored counterpart
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ExportAsReference="true"))
	TObjectPtr<const UMirrorDataTable> MirrorDataTable = nullptr;
};

/**
 * Mirroring Apply/Filter Parameters
 *
 * This struct holds flags for the channels that can be affected during the mirror pass (i.e. Bones, Curves, Attributes).
 *
 * @see FMirroringTrait, FMirroringAdditiveTrait, FAnimNextEvaluationMirroringTask
 */
USTRUCT(BlueprintType, meta = (DisplayName = "Mirroring Trait Apply To Parameters"))
struct FUAFMirroringTraitApplyToParams
{
	GENERATED_BODY()

	// Whether to mirror bone transforms
	UPROPERTY(EditAnywhere, Category = Channels, meta=(DisplayName = "Bones"))
	bool bShouldMirrorBones = true;

	// Whether to mirror animation curves
	UPROPERTY(EditAnywhere, Category = Channels, meta=(DisplayName = "Curves"))
	bool bShouldMirrorCurves = true;

	// Whether to mirror attributes
	UPROPERTY(EditAnywhere, Category = Channels, meta=(DisplayName = "Attributes"))
	bool bShouldMirrorAttributes = true;
};

namespace UE::UAF
{
	// Namespaced alias
	using FMirroringTraitSetupParams = FUAFMirroringTraitSetupParams;
	using FMirroringTraitApplyToParams = FUAFMirroringTraitApplyToParams;
	
	/**
	 * Mirroring Cache
	 *
	 * Holds all precomputed data needed to mirror keyframe output efficiently, so it doesn’t have to be rebuilt every evaluation.
	 * 
	 * @see FAnimNextEvaluationMirroringTask
	 */
	struct FMirroringTraitCache
	{
		// Cached mirror indices map (invalidated on skeletal mesh or mirror data table change)
		TArray<FBoneIndexType> MeshBoneIndexToMirroredMeshBoneIndexMap;
		
		// Cached bind pose rotations (invalidated on skeletal mesh change)
		TArray<FQuat> MeshSpaceReferencePoseRotations;

		// Cached bind pose rotation corrections (invalidated on skeletal mesh change)
		TArray<FQuat> MeshSpaceReferenceRotationCorrections;
		
		// Cached mirror indices map but using FCompactPoseIndex (invalidated on skeletal mesh or mirror data table change)
		TCustomBoneIndexArray<FCompactPoseBoneIndex, FCompactPoseBoneIndex> CompactPoseBoneIndexToMirroredCompactPoseBoneIndexMap;

		// Skeletal mesh used to build this cache
		TWeakObjectPtr<const USkeletalMesh> SkeletalMesh = nullptr;

		// Mirror data table used to build this cache
		TWeakObjectPtr<const UMirrorDataTable> MirrorTable = nullptr;

		/**
		 * True if the cached mirror maps were generated using exactly these assets.
		 * Any mismatch (different mesh/mirror table, size mismatch) means you must rebuild the mirror maps.
		 */
		bool AreMirrorMapsValid(const UE::UAF::FReferencePose& InReferencePose, const TWeakObjectPtr<const UMirrorDataTable>& InMirrorTable, bool bInShouldMirrorBones, bool bInShouldMirrorAttributes) const;

		/**
		 * True if the cached bind/reference-pose data matches this mesh.
		 * Any mismatch (different mesh table, size mismatch)  means you must rebuild the reference-pose arrays.
		 */
		bool IsReferencePoseDataValid(const UE::UAF::FReferencePose& InReferencePose, bool bInShouldMirrorBones) const;

		/**
		 * True if the cache was generated using exactly these assets.
		 * Any mismatch (different mesh/mirror table, size mismatch) means you must rebuild the cache.
		 */
		bool IsValid(const UE::UAF::FReferencePose& InReferencePose, const TWeakObjectPtr<const UMirrorDataTable>& InMirrorTable, bool bInShouldMirrorBones, bool bInShouldMirrorAttributes) const;

		/** Empty arrays and reset ptrs to assets */
		void Clear();
	};
}
	
/**
* Mirroring Task
 * 
 * This pop the top keyframe from the VM keyframe stack, mirrors it, and pushes
 * back the result onto the stack.
 */
USTRUCT()
struct UAFMIRRORING_API FAnimNextEvaluationMirroringTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextEvaluationMirroringTask)

	// Settings to use when doing the mirror pass
	UPROPERTY(VisibleAnywhere, Category = "Setup")
	FUAFMirroringTraitSetupParams Setup;

	UPROPERTY(VisibleAnywhere, Category = "Apply To")
	FUAFMirroringTraitApplyToParams ApplyTo;

	// Cache for bind pose rotation, corrections, and mirror map
	mutable UE::UAF::FMirroringTraitCache Cache;

	// Task entry point
	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

private:

	// Ensures the mirror cache matches the given inputs, rebuilds if stale.
	void EnsureCache(const UE::UAF::FEvaluationVM& VM, const UE::UAF::FReferencePose& InReferencePose) const;
};