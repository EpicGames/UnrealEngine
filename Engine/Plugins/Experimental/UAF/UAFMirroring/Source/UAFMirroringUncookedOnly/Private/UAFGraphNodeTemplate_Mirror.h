// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphUncookedOnlyUtils.h"
#include "AnimNextController.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateIconFinder.h"
#include "Templates/UAFGraphNodeTemplate.h"
#include "MirroringTraitData.h"
#include "Animation/MirrorDataTable.h"

#include "UAFGraphNodeTemplate_Mirror.generated.h"

#define LOCTEXT_NAMESPACE "UAFGraphNodeTemplate_Mirror"

UCLASS()
class UUAFGraphNodeTemplate_Mirror : public UUAFGraphNodeTemplate
{
	GENERATED_BODY()

	UUAFGraphNodeTemplate_Mirror()
	{
		Title = LOCTEXT("MirrorTitle", "Mirror");
		TooltipText = LOCTEXT("MirrorTooltip", "Mirror an input using a mirror data table");
		Category = LOCTEXT("MirrorCategory", "UAF");
		MenuDescription = LOCTEXT("MirrorMenuDesc", "Mirror");
		Color = FLinearColor(FColor(62, 140, 35)); // From UAssetDefinition_DataTable
		Icon = *FSlateIconFinder::FindIconForClass(UMirrorDataTable::StaticClass()).GetIcon();
		Traits =
		{
			TInstancedStruct<UE::UAF::FMirroringTraitData>::Make(),
		};
		DragDropAssetTypes.Add(UMirrorDataTable::StaticClass());
		SetCategoryForPinsInLayout(
			{
				GET_PIN_PATH_STRING_CHECKED(FMirroringTraitSharedData, Input),
				GET_PIN_PATH_STRING_CHECKED(FMirroringTraitSharedData, Setup),
				GET_PIN_PATH_STRING_CHECKED(FMirroringTraitSharedData, ApplyTo),
			},
			FRigVMPinCategory::GetDefaultCategoryName(),
			NodeLayout,
			true);
	}

	virtual void HandleAssetDropped_Implementation(UAnimNextController* Controller, URigVMUnitNode* Node, UObject* Asset) const
	{
		Super::HandleAssetDropped_Implementation(Controller, Node, Asset);

		// @todo: Wish we could use a subtitle instead but looks like the subtitle can't be set via the controller as its not instanced per not but rather queried per template.

		const bool bSetupUndoRedo = !GIsTransacting;
		if (Asset)
		{
			if (bSetupUndoRedo)
			{
				Controller->OpenUndoBracket(LOCTEXT("ConfigureNodeOnDrop", "Configure Node On Drop").ToString());
			}

			Controller->SetNodeTitle(Node, FText::Format(LOCTEXT("NodeTitleFormat", "Mirror using {0}"), FText::FromString(Asset->GetName())).ToString(), bSetupUndoRedo, true, true);

			if (bSetupUndoRedo)
			{
				Controller->CloseUndoBracket();
			}
		}
	}

	virtual void HandlePinDefaultValueChanged_Implementation(UAnimNextController* Controller, URigVMPin* Pin) const
	{
		Super::HandlePinDefaultValueChanged_Implementation(Controller, Pin);

		const bool bSetupUndoRedo = !GIsTransacting;
		if (Pin->GetFName() == GET_MEMBER_NAME_CHECKED(UE::UAF::FMirroringTraitSetupParams, MirrorDataTable))
		{
			if (bSetupUndoRedo)
			{
				Controller->OpenUndoBracket(LOCTEXT("SetNodeTitle", "Set Node Title").ToString());
			}

			FSoftObjectPath Path(Pin->GetDefaultValue());
			Controller->SetNodeTitle(Pin->GetNode(), FText::Format(LOCTEXT("NodeTitleFormat", "Mirror using {0}"), FText::FromString(Path.GetAssetName())).ToString(), bSetupUndoRedo, true, true);

			if (bSetupUndoRedo)
			{
				Controller->CloseUndoBracket();
			}
		}
	}
	
	virtual void HandleAssetRenamed_Implementation(UAnimNextController* Controller, URigVMNode* Node, const FAssetData& AssetData, const FString& OldName) const
	{
		Super::HandleAssetRenamed_Implementation(Controller, Node, AssetData, OldName);

		// It the asset is not loaded, it cant be our asset so we can safely early out
		UObject* Object = AssetData.FastGetAsset();
		if (Object == nullptr)
		{
			return;
		}

		// NOTE: Not making this undo-able as this is an automated process, not a user action on the node specifically

		URigVMPin* Pin = Node->FindPin("MirroringTraitSharedData.Setup.MirrorDataTable");
		if (Pin && Object == Pin->GetDefaultValueObject())
		{
			FSoftObjectPath Path(Object);
			Controller->SetNodeTitle(Node, FText::Format(LOCTEXT("NodeTitleFormat", "Mirror using {0}"), FText::FromString(Path.GetAssetName())).ToString(), false, true, true);
		}
	}
};

#undef LOCTEXT_NAMESPACE