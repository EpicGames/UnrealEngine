// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UAFMirroringUncookedOnly : ModuleRules
{
    public UAFMirroringUncookedOnly(ReadOnlyTargetRules Target) : base(Target)
    {
	    PrivateDependencyModuleNames.AddRange(
		    new string[]
		    {
			    "Core",
			    "CoreUObject",
			    "AnimationCore",
			    "Engine",
			    "UAF",
			    "UAFUncookedOnly",
			    "UAFAnimGraph",
			    "UAFAnimGraphUncookedOnly",
			    "UAFMirroring",
			    "RigVMDeveloper",
			    "RigVM",
			    "Slate",
			    "SlateCore",
		    }
	    );
    }
}