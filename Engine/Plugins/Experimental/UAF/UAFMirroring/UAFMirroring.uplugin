{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "UAF Mirroring",
	"Description": "Keyframe mirroring for UAF",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Plugins": [
		{
			"Name": "UAF",
			"Enabled": true
		},
		{
			"Name": "UAFAnimGraph",
			"Enabled": true
		},
		{
			"Name": "RigVM",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "UAFMirroring",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "UAFMirroringUncookedOnly",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		}
	]
}